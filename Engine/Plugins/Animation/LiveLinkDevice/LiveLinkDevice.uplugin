{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Live Link Device Framework",
	"Description": "Provides interfaces and base classes for implementing Live Link Hub devices",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Hidden": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "LiveLinkDevice",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}
