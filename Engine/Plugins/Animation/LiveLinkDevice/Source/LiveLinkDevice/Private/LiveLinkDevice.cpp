// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkDevice.h"
#include "Engine/Engine.h"
#include "JsonObjectConverter.h"
#include "LiveLinkDefaultColumns.h"
#include "LiveLinkDeviceCapability.h"
#include "LiveLinkDeviceModule.h"
#include "LiveLinkDeviceSubsystem.h"
#include "LiveLinkDeviceStyle.h"
#include "Logging/StructuredLog.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SNullWidget.h"



FText ULiveLinkDevice::GetDisplayName() const
{
	if (Settings)
	{
		return FText::FromString(Settings->DisplayName);
	}
	
	return FText::GetEmpty();
}

TSharedRef<SWidget> ULiveLinkDevice::GenerateWidgetForColumn(const FName InColumnId, const FLiveLinkDeviceWidgetArguments& InArgs)
{
	// Handle generating a health widget for the status column.
	if (InColumnId == UE::LiveLink::DefaultColumn::Status)
	{
		return SNew(SImage)
			.Image_UObject(this, &ULiveLinkDevice::GetHealthImage)
			.ToolTipText_UObject(this, &ULiveLinkDevice::GetHealthText);
	}

	// Check if the capability provides a default widget implementation.
	ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();
	const TMap<FName, TSubclassOf<ULiveLinkDeviceCapability>>& ColumnIdToCapability = Subsystem->GetTableColumnIdToCapability();
	const TSubclassOf<ULiveLinkDeviceCapability>* MaybeCapabilityClass = ColumnIdToCapability.Find(InColumnId);
	if (MaybeCapabilityClass && GetClass()->ImplementsInterface(*MaybeCapabilityClass))
	{
		ULiveLinkDeviceCapability* CapabilityCDO = MaybeCapabilityClass->GetDefaultObject();
		if (TSharedPtr<SWidget> CapabilityWidget = CapabilityCDO->GenerateWidgetForColumn(InColumnId, InArgs, this))
		{
			return CapabilityWidget.ToSharedRef();
		}
	}

	// Neither your device class nor the capability created a widget for this column.
	return SNullWidget::NullWidget;
}

const FSlateBrush* ULiveLinkDevice::GetHealthImage() const
{
	switch (GetDeviceHealth())
	{
		case EDeviceHealth::Good: return FLiveLinkDeviceStyle::Get()->GetBrush("LiveLinkHub.Devices.Status.Good");
		case EDeviceHealth::Info: return FLiveLinkDeviceStyle::Get()->GetBrush("LiveLinkHub.Devices.Status.Info");
		case EDeviceHealth::Warning: return FLiveLinkDeviceStyle::Get()->GetBrush("LiveLinkHub.Devices.Status.Warning");
		case EDeviceHealth::Error: return FLiveLinkDeviceStyle::Get()->GetBrush("LiveLinkHub.Devices.Status.Error");
		default: return nullptr;
	}
}

void ULiveLinkDevice::InternalDeviceAdded(const FGuid InDeviceGuid, ULiveLinkDeviceSettings* InSettings)
{
	DeviceGuid = InDeviceGuid;
	Settings = InSettings;

	OnDeviceAdded();
}
