// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkDeviceCapability_Connection.h"
#include "Engine/Engine.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LiveLinkDefaultColumns.h"
#include "LiveLinkDevice.h"
#include "LiveLinkDeviceSubsystem.h"
#include "Styling/StyleColors.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "LiveLinkDevice"


class SLiveLinkConnection_HardwareId : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLiveLinkConnection_HardwareId) {}
		SLATE_ATTRIBUTE(FText, Text)
		SLATE_ATTRIBUTE(bool, IsReadOnly)

		/** Must be propagated for editable child widgets to work. */
		SLATE_EVENT(FIsSelected, IsDeviceRowSelected)

		SLATE_EVENT(FOnTextCommitted, OnTextCommitted)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SInlineEditableTextBlock)
			.IsReadOnly(InArgs._IsReadOnly)
			.IsSelected(InArgs._IsDeviceRowSelected)
			.OnTextCommitted(InArgs._OnTextCommitted)
			.Text(InArgs._Text)
			.ToolTipText(InArgs._Text)
		];
	}

protected:
	TAttribute<ELiveLinkDeviceConnectionStatus> ConnectionStatus;
};


ULiveLinkDeviceCapability_Connection::ULiveLinkDeviceCapability_Connection()
	: Column_HardwareId(RegisterTableColumn("HardwareId"))
{
	RegisterGlobalTableColumnHandler(UE::LiveLink::DefaultColumn::Machine);
}

SHeaderRow::FColumn::FArguments& ULiveLinkDeviceCapability_Connection::GenerateHeaderForColumn(
	const FName InColumnId,
	SHeaderRow::FColumn::FArguments& InArgs
)
{
	if (InColumnId == Column_HardwareId)
	{
		return InArgs
			.DefaultLabel(LOCTEXT("HardwareID_ColumnHeader_Label", "Hardware ID"))
			.DefaultTooltip(LOCTEXT("HardwareID_ColumnHeader_Tooltip", "Device hardware identifier"))
			.HeaderComboVisibility(EHeaderComboVisibility::Ghosted)
			.OnGetMenuContent_UObject(this, &ULiveLinkDeviceCapability_Connection::Header_GetMenuContent)
			.FillWidth(0.75f);
	}

	return Super::GenerateHeaderForColumn(InColumnId, InArgs);
}


TSharedPtr<SWidget> ULiveLinkDeviceCapability_Connection::GenerateWidgetForColumn(
	const FName InColumnId,
	const FLiveLinkDeviceWidgetArguments& InArgs,
	ULiveLinkDevice* InDevice
)
{
	return SNew(SLiveLinkConnection_HardwareId)
		.IsDeviceRowSelected(InArgs.IsRowSelected)
		.IsReadOnly_Lambda(
			[WeakDevice = TWeakObjectPtr<ULiveLinkDevice>(InDevice)]
			()
			{
				if (ULiveLinkDevice* Device = WeakDevice.Get())
				{
					return !ILiveLinkDeviceCapability_Connection::Execute_CanSetHardwareId(Device);
				}
				else
				{
					return true;
				}
			}
		)
		.Text_Lambda(
			[WeakDevice = TWeakObjectPtr<ULiveLinkDevice>(InDevice)]
			()
			{
				if (ULiveLinkDevice* Device = WeakDevice.Get())
				{
					const FString HardwareId = ILiveLinkDeviceCapability_Connection::Execute_GetHardwareId(Device);
					return FText::FromString(HardwareId);
				}
				else
				{
					return FText();
				}
			}
		)
		.OnTextCommitted_Lambda(
			[WeakDevice = TWeakObjectPtr<ULiveLinkDevice>(InDevice)]
			(const FText& InNewText, ETextCommit::Type InCommitType)
			{
				if (ULiveLinkDevice* Device = WeakDevice.Get())
				{
					ILiveLinkDeviceCapability_Connection::Execute_SetHardwareId(Device, InNewText.ToString());
				}
			}
		);
}

TSharedRef<SWidget> ULiveLinkDeviceCapability_Connection::Header_GetMenuContent()
{
	const bool bShouldCloseAfterSelection_true = true;
	FMenuBuilder MenuBuilder(bShouldCloseAfterSelection_true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("HardwareID_ColumnHeader_ConnectAll_Label", "Connect All Devices"),
		TAttribute<FText>(),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateUObject(this, &ULiveLinkDeviceCapability_Connection::ConnectAllDevices))
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("HardwareID_ColumnHeader_DisconnectAll_Label", "Disconnect All Devices"),
		TAttribute<FText>(),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateUObject(this, &ULiveLinkDeviceCapability_Connection::DisconnectAllDevices))
	);

	return MenuBuilder.MakeWidget();
}


void ULiveLinkDeviceCapability_Connection::ConnectAllDevices()
{
	ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();

	TArray<ULiveLinkDevice*> ConnectibleDevices;
	Subsystem->GetDevicesByCapability(ULiveLinkDeviceCapability_Connection::StaticClass(), ConnectibleDevices);
	for (ULiveLinkDevice* Device : ConnectibleDevices)
	{
		if (ensure(Device->Implements<ULiveLinkDeviceCapability_Connection>()))
		{
			ILiveLinkDeviceCapability_Connection::Execute_Connect(Device);
		}
	}
}


void ULiveLinkDeviceCapability_Connection::DisconnectAllDevices()
{
	ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();

	TArray<ULiveLinkDevice*> ConnectibleDevices;
	Subsystem->GetDevicesByCapability(ULiveLinkDeviceCapability_Connection::StaticClass(), ConnectibleDevices);
	for (ULiveLinkDevice* Device : ConnectibleDevices)
	{
		if (ensure(Device->Implements<ULiveLinkDeviceCapability_Connection>()))
		{
			ILiveLinkDeviceCapability_Connection::Execute_Disconnect(Device);
		}
	}
}


ILiveLinkDeviceCapability_Connection::ILiveLinkDeviceCapability_Connection()
	: ConnectionDelegate(NewObject<UConnectionDelegate>())
{
}


UConnectionDelegate* ILiveLinkDeviceCapability_Connection::GetConnectionDelegate_Implementation()
{
	return ConnectionDelegate.Get();
}


void ILiveLinkDeviceCapability_Connection::SetConnectionStatus(ELiveLinkDeviceConnectionStatus InStatus)
{
	ConnectionDelegate->ConnectionChanged.Broadcast(InStatus);

	if (IsInGameThread())
	{
		ConnectionDelegate->ConnectionChangedDynamic.Broadcast(InStatus);
	}
	else
	{
		ExecuteOnGameThread(TEXT("InvokeConnectionDelegate"), [Delegate = TWeakObjectPtr<UConnectionDelegate>(ConnectionDelegate.Get()), Status = InStatus]()
		{
			if (TStrongObjectPtr<UConnectionDelegate> StrongDelegate = Delegate.Pin())
			{
				StrongDelegate->ConnectionChangedDynamic.Broadcast(Status);
			}
		});
	}
}


#undef LOCTEXT_NAMESPACE
