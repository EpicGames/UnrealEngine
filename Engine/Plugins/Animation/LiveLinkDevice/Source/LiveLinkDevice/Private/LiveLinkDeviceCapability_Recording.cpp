// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkDeviceCapability_Recording.h"
#include "Engine/Engine.h"
#include "LiveLinkDefaultColumns.h"
#include "LiveLinkDevice.h"
#include "LiveLinkDeviceStyle.h"
#include "LiveLinkDeviceSubsystem.h"
#include "LiveLinkHub/ILiveLinkRecordingSession.h"
#include "Widgets/Images/SImage.h"

ULiveLinkDeviceCapability_Recording::ULiveLinkDeviceCapability_Recording()
	: Column_RecordingStatus(RegisterTableColumn("RecordingStatus"))
{
	RegisterGlobalTableColumnHandler(UE::LiveLink::DefaultColumn::Action);
}

class SLiveLinkCapability_Recording : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLiveLinkCapability_Recording) {}
		SLATE_ATTRIBUTE(bool, IsRecording)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs)
	{
		IsRecording = InArgs._IsRecording;

		ChildSlot
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SImage)
			.Image_Raw(this, &SLiveLinkCapability_Recording::GetImage)
			.ColorAndOpacity_Raw(this, &SLiveLinkCapability_Recording::GetColorAndOpacity)
		];
	}

	const FSlateBrush* GetImage() const
	{
		return IsRecording.Get()
			? FLiveLinkDeviceStyle::Get()->GetBrush("Record")
			: FLiveLinkDeviceStyle::Get()->GetBrush("Record.Monochrome");
	}

	FSlateColor GetColorAndOpacity() const
	{
		return IsRecording.Get() ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground();
	}

protected:
	TAttribute<bool> IsRecording;
};


void ULiveLinkDeviceCapability_Recording::OnDeviceSubsystemInitialized()
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkRecordingSession::GetModularFeatureName()))
	{
		ILiveLinkRecordingSession::Get().OnRecordingStarted().AddUObject(
			this, &ULiveLinkDeviceCapability_Recording::HandleRecordingStarted
		);

		ILiveLinkRecordingSession::Get().OnRecordingStopped().AddUObject(
			this, &ULiveLinkDeviceCapability_Recording::HandleRecordingStopped
		);
	}
}


void ULiveLinkDeviceCapability_Recording::OnDeviceSubsystemDeinitializing()
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (ModularFeatures.IsModularFeatureAvailable(ILiveLinkRecordingSession::GetModularFeatureName()))
	{
		ILiveLinkRecordingSession::Get().OnRecordingStarted().RemoveAll(this);
		ILiveLinkRecordingSession::Get().OnRecordingStopped().RemoveAll(this);
	}
}


SHeaderRow::FColumn::FArguments& ULiveLinkDeviceCapability_Recording::GenerateHeaderForColumn(
	const FName InColumnId,
	SHeaderRow::FColumn::FArguments& InArgs
)
{
	if (InColumnId == Column_RecordingStatus)
	{
		return InArgs.DefaultLabel(FText::FromString(" "))
			.DefaultTooltip(FText::FromString("Device recording status"))
			.FillSized(30.0f);
	}

	return Super::GenerateHeaderForColumn(InColumnId, InArgs);
}


TSharedPtr<SWidget> ULiveLinkDeviceCapability_Recording::GenerateWidgetForColumn(
	const FName InColumnId,
	const FLiveLinkDeviceWidgetArguments& InArgs,
	ULiveLinkDevice* InDevice
)
{
	return SNew(SLiveLinkCapability_Recording)
		.IsRecording_Lambda(
			[WeakDevice = TWeakObjectPtr<ULiveLinkDevice>(InDevice)]
			()
			{
				if (ULiveLinkDevice* Device = WeakDevice.Get())
				{
					return ILiveLinkDeviceCapability_Recording::Execute_IsRecording(Device);
				}

				return false;
			}
		);
}

void ULiveLinkDeviceCapability_Recording::HandleRecordingStarted()
{
	ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();

	TArray<ULiveLinkDevice*> RecordingDevices;
	Subsystem->GetDevicesByCapability(ULiveLinkDeviceCapability_Recording::StaticClass(), RecordingDevices);

	for (ULiveLinkDevice* Device : RecordingDevices)
	{
		ILiveLinkDeviceCapability_Recording::Execute_StartRecording(Device);
	}
}


void ULiveLinkDeviceCapability_Recording::HandleRecordingStopped()
{
	ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();

	TArray<ULiveLinkDevice*> RecordingDevices;
	Subsystem->GetDevicesByCapability(ULiveLinkDeviceCapability_Recording::StaticClass(), RecordingDevices);

	for (ULiveLinkDevice* Device : RecordingDevices)
	{
		ILiveLinkDeviceCapability_Recording::Execute_StopRecording(Device);
	}
}
