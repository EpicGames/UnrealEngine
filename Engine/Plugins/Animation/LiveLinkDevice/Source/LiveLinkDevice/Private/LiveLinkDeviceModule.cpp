// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkDeviceModule.h"
#include "DetailsViewArgs.h"
#include "LiveLinkDeviceSettingsDetailCustomization.h"
#include "Engine/Engine.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LiveLinkDevice.h"
#include "LiveLinkDeviceStyle.h"
#include "LiveLinkDeviceSubsystem.h"
#include "Logging/StructuredLog.h"
#include "PropertyEditorModule.h"
#include "SPositiveActionButton.h"
#include "Widgets/Docking/SDockTab.h"


DEFINE_LOG_CATEGORY(LogLiveLinkDevice);

#define LOCTEXT_NAMESPACE "LiveLinkDevice"


IMPLEMENT_MODULE(FLiveLinkDeviceModule, LiveLinkDevice);


//////////////////////////////////////////////////////////////////////////


void FLiveLinkDeviceModule::StartupModule()
{
	FLiveLinkDeviceStyle::Initialize();

	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		RegisteredCustomClassLayoutName = ULiveLinkDeviceSettings::StaticClass()->GetFName();
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout(
			RegisteredCustomClassLayoutName,
			FOnGetDetailCustomizationInstance::CreateStatic(&FLiveLinkDeviceSettingsDetailCustomization::MakeInstance)
		);
	}
}


void FLiveLinkDeviceModule::ShutdownModule()
{
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor") && !RegisteredCustomClassLayoutName.IsNone())
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout(RegisteredCustomClassLayoutName);
	}

	FLiveLinkDeviceStyle::Shutdown();
}


void FLiveLinkDeviceModule::DeviceSelectionChanged(ULiveLinkDevice* InSelectedDevice)
{
	WeakSelectedDevice = InSelectedDevice;

	if (DetailsView)
	{
		DetailsView->SetObject(InSelectedDevice ? InSelectedDevice->GetDeviceSettings() : nullptr);
	}

	OnDeviceSelectionChangedDelegate.Broadcast(InSelectedDevice);
}


void FLiveLinkDeviceModule::CreateDeviceMenuEntries(FMenuBuilder& MenuBuilder)
{
	ULiveLinkDeviceSubsystem* DeviceSubsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();
	check(DeviceSubsystem);

	const TSet<TSubclassOf<ULiveLinkDevice>>& DeviceClasses = DeviceSubsystem->GetKnownDeviceClasses();

	MenuBuilder.BeginSection("DevicesSection", LOCTEXT("DevicesSectionHeading", "Live Link Devices"));

	for (const TSubclassOf<ULiveLinkDevice>& DeviceClass : DeviceClasses)
	{
		if (DeviceClass->HasAnyClassFlags(CLASS_Abstract | CLASS_NotPlaceable))
		{
			continue;
		}

		FText ToolTip = DeviceClass->GetToolTipText();
		if (ToolTip.IsEmpty())
		{
			ToolTip = FText::FromString(DeviceClass->GetPathName());
		}

		MenuBuilder.AddMenuEntry(
			DeviceClass->GetDisplayNameText(),
			ToolTip,
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda(
				[DeviceSubsystem, DeviceClass]
				{
					DeviceSubsystem->CreateDeviceOfClass(DeviceClass);
				}
			)),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}

	MenuBuilder.EndSection();
}

#undef LOCTEXT_NAMESPACE
