// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILiveLinkDeviceModule.h"

#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"


class FMenuBuilder;
class FSpawnTabArgs;
class IDetailsView;
class SDockTab;
class SLiveLinkDeviceTable;
class SWidget;

class FLiveLinkDeviceModule : public ILiveLinkDeviceModule
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	FOnDeviceSelectionChangedDelegate& OnSelectionChanged() override { return OnDeviceSelectionChangedDelegate; }

	//~ Begin ILiveLinkDeviceModule interface
	virtual void CreateDeviceMenuEntries(FMenuBuilder& Builder) override;
	//~ End ILiveLinkDeviceModule interface

private:
	void DeviceSelectionChanged(ULiveLinkDevice* InSelectedDevice);

	TSharedPtr<IDetailsView> DetailsView;
	TWeakObjectPtr<ULiveLinkDevice> WeakSelectedDevice;
	FName RegisteredCustomClassLayoutName;

	FOnDeviceSelectionChangedDelegate OnDeviceSelectionChangedDelegate;
};
