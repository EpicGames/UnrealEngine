// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkDeviceSettingsDetailCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/Engine.h"
#include "IDetailPropertyRow.h"
#include "LiveLinkDevice.h"
#include "LiveLinkDeviceSubsystem.h"
#include "Widgets/Input/SEditableTextBox.h"


#define LOCTEXT_NAMESPACE "LiveLinkDevice"


TSharedRef<IDetailCustomization> FLiveLinkDeviceSettingsDetailCustomization::MakeInstance()
{
	return MakeShareable(new FLiveLinkDeviceSettingsDetailCustomization());
}

void FLiveLinkDeviceSettingsDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DisplayNameHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULiveLinkDeviceSettings, DisplayName));

	if (!DisplayNameHandle->IsValidHandle())
	{
		return;
	}

	// Resolve the owning device so we can exclude it from the uniqueness check.
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	if (ObjectsBeingCustomized.Num() > 0)
	{
		if (ULiveLinkDeviceSettings* Settings = Cast<ULiveLinkDeviceSettings>(ObjectsBeingCustomized[0].Get()))
		{
			WeakDevice = Settings->GetTypedOuter<ULiveLinkDevice>();
		}
	}

	DetailBuilder.EditCategory(
		DisplayNameHandle->GetDefaultCategoryName(),
		FText::GetEmpty(),
		ECategoryPriority::Important  // sorts above Default-priority categories
	);

	IDetailPropertyRow* Row = DetailBuilder.EditDefaultProperty(DisplayNameHandle.ToSharedRef());
	Row->OverrideResetToDefault(FResetToDefaultOverride::Hide());

	Row->CustomWidget()
		.NameContent()
		[
			DisplayNameHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SEditableTextBox)
			.Text(this, &FLiveLinkDeviceSettingsDetailCustomization::GetDisplayName)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.OnVerifyTextChanged(this, &FLiveLinkDeviceSettingsDetailCustomization::OnVerifyDisplayNameChanged)
			.OnTextCommitted(this, &FLiveLinkDeviceSettingsDetailCustomization::SetDisplayName)
			.SelectAllTextWhenFocused(true)
		];
}

FText FLiveLinkDeviceSettingsDetailCustomization::GetDisplayName() const
{
	if (DisplayNameHandle.IsValid())
	{
		FText CurrentValue;
		DisplayNameHandle->GetValueAsDisplayText(CurrentValue);
		return CurrentValue;
	}

	return FText::GetEmpty();
}

bool FLiveLinkDeviceSettingsDetailCustomization::OnVerifyDisplayNameChanged(const FText& InText, FText& OutErrorMessage) const
{
	const FString TrimmedName = InText.ToString().TrimStartAndEnd();

	if (TrimmedName.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("LiveLinkDeviceSettings_EmptyName", "Display name cannot be empty.");
		return false;
	}

	ULiveLinkDeviceSubsystem* Subsystem = GEngine ? GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>() : nullptr;
	if (!Subsystem)
	{
		return true;
	}

	const ULiveLinkDevice* CurrentDevice = WeakDevice.Get();

	for (const TPair<FGuid, TObjectPtr<ULiveLinkDevice>>& DevicePair : Subsystem->GetDeviceMap())
	{
		if (DevicePair.Value == CurrentDevice)
		{
			continue;
		}

		if (DevicePair.Value->GetDisplayName().ToString() == TrimmedName)
		{
			OutErrorMessage = LOCTEXT("LiveLinkDeviceSettings_DuplicateName", "A device with that name already exists.");
			return false;
		}
	}

	return true;
}

void FLiveLinkDeviceSettingsDetailCustomization::SetDisplayName(const FText& InCommittedText, ETextCommit::Type InCommitType)
{
	if (DisplayNameHandle.IsValid())
	{
		DisplayNameHandle->SetValue(InCommittedText.ToString().TrimStartAndEnd());
	}
}

#undef LOCTEXT_NAMESPACE
