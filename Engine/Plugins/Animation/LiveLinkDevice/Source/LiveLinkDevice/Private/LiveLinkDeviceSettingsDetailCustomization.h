// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"
#include "PropertyHandle.h"

class ULiveLinkDevice;
class ULiveLinkDeviceSettings;
class ULiveLinkDeviceSubsystem;


/**
 * Detail customization for ULiveLinkDeviceSettings.
 * Validates that DisplayName is unique across all devices.
 */
class FLiveLinkDeviceSettingsDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization interface

private:

	FText GetDisplayName() const;
	bool OnVerifyDisplayNameChanged(const FText& InText, FText& OutErrorMessage) const;
	void SetDisplayName(const FText& InCommittedText, ETextCommit::Type InCommitType);

	TWeakObjectPtr<ULiveLinkDevice> WeakDevice;
	TSharedPtr<IPropertyHandle> DisplayNameHandle;
};
