// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkDeviceStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StyleColors.h"


TSharedPtr<FSlateStyleSet> FLiveLinkDeviceStyle::StyleSet;


void FLiveLinkDeviceStyle::Initialize()
{
	if (!ensureMsgf(!StyleSet, TEXT("FLiveLinkDeviceStyle already initialized")))
	{
		return;
	}

	static const FVector2D Icon16x16(16.0f, 16.0f);
	static const FVector2D Icon20x20(20.0f, 20.0f);

	const FString PluginContentRoot =
		IPluginManager::Get().FindPlugin("LiveLinkDevice")->GetBaseDir() / TEXT("Resources");

	StyleSet = MakeShared<FSlateStyleSet>(GetStyleSetName());

	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	StyleSet->Set("Record", new FSlateVectorImageBrush(PluginContentRoot / "PlayControlsRecord.svg", Icon16x16));
	StyleSet->Set("Record.Monochrome", new FSlateVectorImageBrush(PluginContentRoot / "PlayControlsRecord_Monochrome.svg", Icon16x16));

	StyleSet->Set("LiveLinkHub.Devices.Icon", new FSlateVectorImageBrush(PluginContentRoot / "Devices.svg", Icon16x16));

	StyleSet->Set("LiveLinkHub.Devices.Status.Good", new FSlateVectorImageBrush(StyleSet->RootToCoreContentDir("Starship/Common/check-circle-solid.svg"), Icon16x16, FStyleColors::Success));
	StyleSet->Set("LiveLinkHub.Devices.Status.Info", new FSlateVectorImageBrush(StyleSet->RootToCoreContentDir("Starship/Common/info-circle-solid.svg"), Icon16x16));
	StyleSet->Set("LiveLinkHub.Devices.Status.Warning", new FSlateVectorImageBrush(StyleSet->RootToContentDir("Starship/Common/AlertTriangleSolid.svg"), Icon16x16, FStyleColors::Warning));
	StyleSet->Set("LiveLinkHub.Devices.Status.Error", new FSlateVectorImageBrush(StyleSet->RootToContentDir("Starship/Common/AlertTriangleSolid.svg"), Icon16x16, FStyleColors::Error));

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet);
}


void FLiveLinkDeviceStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet);
	StyleSet.Reset();
}


TSharedPtr<ISlateStyle> FLiveLinkDeviceStyle::Get()
{
	return StyleSet;
}
