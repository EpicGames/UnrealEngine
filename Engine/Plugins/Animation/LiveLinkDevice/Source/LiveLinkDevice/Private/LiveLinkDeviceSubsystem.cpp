// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkDeviceSubsystem.h"
#include "Logging/StructuredLog.h"
#include "LiveLinkDevice.h"
#include "LiveLinkDeviceCapability.h"
#include "LiveLinkDeviceModule.h"
#include "LiveLinkHubSessionExtraData_Device.h"


void ULiveLinkDeviceSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	RegisterExtraDataHandler();

	// Enumerate capability classes
	{
		TArray<UClass*> Capabilities;
		GetDerivedClasses(ULiveLinkDeviceCapability::StaticClass(), Capabilities);

		for (UClass* Capability : Capabilities)
		{
			UE_LOGFMT(LogLiveLinkDevice, Verbose, "Discovered device capability {Name}", Capability->GetFName());
			RegisterCapabilityClass(Capability);
		}
	}

	// Enumerate device classes
	{
		TArray<UClass*> DeviceClasses;
		GetDerivedClasses(ULiveLinkDevice::StaticClass(), DeviceClasses);

		for (UClass* DeviceClass : DeviceClasses)
		{
			UE_LOGFMT(LogLiveLinkDevice, Verbose, "Discovered device class {Name}", DeviceClass->GetFName());
			RegisterDeviceClass(DeviceClass);
		}
	}

	// Dispatch initialized calls to capability CDOs.
	for (TSubclassOf<ULiveLinkDeviceCapability>& Capability : KnownCapabilities)
	{
		ULiveLinkDeviceCapability* CapabilityCDO = Capability.GetDefaultObject();
		CapabilityCDO->OnDeviceSubsystemInitialized();
	}
}


void ULiveLinkDeviceSubsystem::Deinitialize()
{
	UnregisterExtraDataHandler();

	// Dispatch de-initializing calls to capability CDOs.
	for (TSubclassOf<ULiveLinkDeviceCapability>& Capability : KnownCapabilities)
	{
		ULiveLinkDeviceCapability* CapabilityCDO = Capability.GetDefaultObject();
		CapabilityCDO->OnDeviceSubsystemDeinitializing();
	}

	// Remove all existing devices.
	TArray<FGuid> DeviceIds;
	Devices.GetKeys(DeviceIds);
	for (const FGuid& DeviceId : DeviceIds)
	{
		RemoveDevice(DeviceId);
	}

	Super::Deinitialize();
}


TSubclassOf<ULiveLinkHubSessionExtraData> ULiveLinkDeviceSubsystem::GetExtraDataClass() const
{
	return ULiveLinkHubSessionExtraData_Device::StaticClass();
}


void ULiveLinkDeviceSubsystem::OnExtraDataSessionSaving(ULiveLinkHubSessionExtraData* InExtraData)
{
	ULiveLinkHubSessionExtraData_Device* DeviceData = CastChecked<ULiveLinkHubSessionExtraData_Device>(InExtraData);

	DeviceData->Devices.Empty();

	for (const TPair<FGuid, TObjectPtr<ULiveLinkDevice>>& DevicePair : Devices)
	{
		DeviceData->Devices.Emplace(FLiveLinkDevicePreset{
			.DeviceGuid = DevicePair.Key,
			.DeviceClass = DevicePair.Value->GetClass(),
			.DeviceSettings = DevicePair.Value->GetDeviceSettings(),
			.DeviceCapabilityData = DevicePair.Value->OnSaveDeviceCapabilityData()
		});
	}
}


void ULiveLinkDeviceSubsystem::OnExtraDataSessionLoaded(const ULiveLinkHubSessionExtraData* InExtraData)
{
	// Remove all existing devices.
	{
		TArray<FGuid> AllDeviceGuids;
		AllDeviceGuids.Reserve(Devices.Num());
		Devices.GetKeys(AllDeviceGuids);

		for (const FGuid& DeviceGuid : AllDeviceGuids)
		{
			RemoveDevice(DeviceGuid);
		}
	}

	if (InExtraData == nullptr)
	{
		// New/loaded session has no saved device data.
		return;
	}

	const ULiveLinkHubSessionExtraData_Device* DeviceData = CastChecked<ULiveLinkHubSessionExtraData_Device>(InExtraData);

	for (const FLiveLinkDevicePreset& DevicePreset : DeviceData->Devices)
	{
		UE_LOGFMT(LogLiveLinkDevice, Log, "Restoring saved Live Link device with ID {Guid}", DevicePreset.DeviceGuid);

		if (!DevicePreset.DeviceClass)
		{
			UE_LOGFMT(LogLiveLinkDevice, Error, "Device class missing");
			continue;
		}

		if (!DevicePreset.DeviceSettings)
		{
			UE_LOGFMT(LogLiveLinkDevice, Error, "Device settings missing");
			continue;
		}

		FCreateResult CreateResult = InternalCreateDeviceOfClass(
			DevicePreset.DeviceClass,
			DevicePreset.DeviceSettings,
			DevicePreset.DeviceGuid
		);

		if (CreateResult.HasValue())
		{
			FLiveLinkDeviceCreateResult Result = CreateResult.StealValue();
			Result.Device->OnLoadDeviceCapabilityData(DevicePreset.DeviceCapabilityData);
		}
		else
		{
			UE_LOGFMT(LogLiveLinkDevice, Error, "Failed to restore device");
		}
	}
}


ULiveLinkDeviceSubsystem::FCreateResult ULiveLinkDeviceSubsystem::CreateDeviceOfClass(
	TSubclassOf<ULiveLinkDevice> InDeviceClass,
	ULiveLinkDeviceSettings* InDeviceSettingsTemplate /* = nullptr */
)
{
	return InternalCreateDeviceOfClass(InDeviceClass, InDeviceSettingsTemplate);
}


ULiveLinkDeviceSubsystem::FCreateResult ULiveLinkDeviceSubsystem::InternalCreateDeviceOfClass(
	TSubclassOf<ULiveLinkDevice> InDeviceClass,
	ULiveLinkDeviceSettings* InDeviceSettingsTemplate /* = nullptr */,
	TOptional<FGuid> InDeviceGuid /* = {} */
)
{
	UE_LOGFMT(LogLiveLinkDevice, Log, "Creating new Live Link device of class {DeviceClass}", InDeviceClass->GetName());

	if (!InDeviceGuid)
	{
		InDeviceGuid = FGuid::NewGuid();
	}
	else if (InDeviceGuid == FGuid())
	{
		InDeviceGuid = FGuid::NewGuid();
		UE_LOGFMT(LogLiveLinkDevice, Warning, "Tried to create device with unset GUID; assigning new GUID {DeviceGuid}", InDeviceGuid);
	}

	const FGuid& DeviceGuid = InDeviceGuid.GetValue();

	ULiveLinkDevice* NewDevice = NewObject<ULiveLinkDevice>(this, InDeviceClass);
	if (!ensure(NewDevice))
	{
		UE_LOGFMT(LogLiveLinkDevice, Error, "Failed to create ULiveLinkDevice");
		return MakeError();
	}

	TSubclassOf<ULiveLinkDeviceSettings> SettingsClass = NewDevice->GetSettingsClass();
	ULiveLinkDeviceSettings* NewSettings = nullptr;
	if (InDeviceSettingsTemplate)
	{
		TSubclassOf<ULiveLinkDeviceSettings> TemplateClass = InDeviceSettingsTemplate->GetClass();
		if (TemplateClass != SettingsClass)
		{
			UE_LOGFMT(LogLiveLinkDevice, Error,
				"Settings template is of wrong class (got {TemplateClass}, expected {DeviceClass})",
				TemplateClass->GetName(),
				SettingsClass->GetName());

			return MakeError();
		}

		NewSettings = DuplicateObject(InDeviceSettingsTemplate, NewDevice);
	}
	else
	{
		NewSettings = NewObject<ULiveLinkDeviceSettings>(NewDevice, SettingsClass);
	}

	if (!ensure(NewSettings))
	{
		UE_LOGFMT(LogLiveLinkDevice, Error, "Failed to create ULiveLinkDeviceSettings");
		return MakeError();
	}

	NewSettings->DisplayName = GenerateUniqueDisplayName(NewSettings->DisplayName);

	InternalAddDevice(DeviceGuid, NewDevice, NewSettings);

	return MakeValue(DeviceGuid, NewDevice);
}


void ULiveLinkDeviceSubsystem::InternalAddDevice(
	FGuid InGuid,
	ULiveLinkDevice* InDevice,
	ULiveLinkDeviceSettings* InSettings
)
{
	check(InDevice);
	check(!Devices.Find(InGuid));

	UE_LOGFMT(LogLiveLinkDevice, Log,
		"ULiveLinkDeviceSubsystem: Adding device {DeviceName} with ID {InGuid}.",
		InDevice->GetFName(), InGuid);

	Devices.Add(InGuid, InDevice);
	DevicesByClass.Add(InDevice->GetClass(), InDevice);

	InDevice->InternalDeviceAdded(InGuid, InSettings);

	OnDeviceAddedDelegate.Broadcast(InGuid, InDevice);
}

void ULiveLinkDeviceSubsystem::InternalRemoveDevice(FGuid InDeviceId, ULiveLinkDevice* InDevice)
{
	check(InDevice);
	check(Devices.Find(InDeviceId));
	check(*Devices.Find(InDeviceId) == InDevice);

	UE_LOGFMT(LogLiveLinkDevice, Log,
		"ULiveLinkDeviceSubsystem: Removing device {DeviceName} with ID {DeviceGuid}.",
		InDevice->GetFName(), InDeviceId);

	Devices.Remove(InDeviceId);
	DevicesByClass.Remove(InDevice->GetClass(), InDevice);

	InDevice->OnDeviceRemoved();

	OnDeviceRemovedDelegate.Broadcast(InDeviceId, InDevice);
}


void ULiveLinkDeviceSubsystem::RemoveDevice(FGuid InDeviceId)
{
	if (TObjectPtr<ULiveLinkDevice>* MaybeDevice = Devices.Find(InDeviceId); ensure(MaybeDevice))
	{
		TObjectPtr<ULiveLinkDevice> Device = *MaybeDevice;
		InternalRemoveDevice(InDeviceId, Device);
	}
	else
	{
		UE_LOGFMT(LogLiveLinkDevice, Error,
			"ULiveLinkDeviceSubsystem: Failed to remove device with ID {DeviceGuid}.",
			InDeviceId);
	}
}


void ULiveLinkDeviceSubsystem::RemoveDevice(ULiveLinkDevice* InDevice)
{
	check(InDevice);

	if (const FGuid* MaybeId = Devices.FindKey(InDevice); ensure(MaybeId))
	{
		const FGuid DeviceId = *MaybeId;
		InternalRemoveDevice(DeviceId, InDevice);
	}
	else
	{
		UE_LOGFMT(LogLiveLinkDevice, Error,
			"ULiveLinkDeviceSubsystem: Failed to remove device {DeviceName}.",
			InDevice->GetFName());
	}
}


void ULiveLinkDeviceSubsystem::GetDevicesByClass(TSubclassOf<ULiveLinkDevice> DeviceClass, TArray<ULiveLinkDevice*>& OutDevices) const
{
	DevicesByClass.MultiFind(DeviceClass, OutDevices);
}


void ULiveLinkDeviceSubsystem::GetDevicesByCapability(TSubclassOf<ULiveLinkDeviceCapability> Capability, TArray<ULiveLinkDevice*>& OutDevices) const
{
	TArray<TSubclassOf<ULiveLinkDevice>> ClassesImplementingCapability;
	DeviceClassesByCapability.MultiFind(Capability, ClassesImplementingCapability);
	for (TSubclassOf<ULiveLinkDevice> Class : ClassesImplementingCapability)
	{
		GetDevicesByClass(Class, OutDevices);
	}
}

bool ULiveLinkDeviceSubsystem::IsDisplayNameInUse(const FString& InDisplayName) const
{
	for (const TPair<FGuid, TObjectPtr<ULiveLinkDevice>>& DevicePair : Devices)
	{
		if (DevicePair.Value->GetDisplayName().ToString() == InDisplayName)
		{
			return true;
		}
	}

	return false;
}

FString ULiveLinkDeviceSubsystem::GenerateUniqueDisplayName(const FString& InBaseName) const
{
	if (!IsDisplayNameInUse(InBaseName))
	{
		return InBaseName;
	}

	int32 Suffix = 1;
	FString Candidate = FString::Printf(TEXT("%s %d"), *InBaseName, Suffix);

	while (IsDisplayNameInUse(Candidate))
	{
		++Suffix;
		Candidate = FString::Printf(TEXT("%s %d"), *InBaseName, Suffix);
	}

	return Candidate;
}

void ULiveLinkDeviceSubsystem::RegisterCapabilityClass(TSubclassOf<ULiveLinkDeviceCapability> InCapability)
{
	if (!ensure(InCapability))
	{
		return;
	}

	if (KnownCapabilities.Find(InCapability))
	{
		ensureMsgf(false, TEXT("Tried to register duplicate capability"));
		return;
	}

	KnownCapabilities.Add(InCapability);

	// We want to encourage shallow inheritance.
	if (InCapability->GetSuperClass() != ULiveLinkDeviceCapability::StaticClass())
	{
		UE_LOGFMT(
			LogLiveLinkDevice,
			Warning,
			"Device capability {Name} inherits from another capability; this is strongly discouraged.",
			InCapability->GetFName()
		);
	}

	ULiveLinkDeviceCapability* CapabilityCDO = InCapability->GetDefaultObject<ULiveLinkDeviceCapability>();

	for (const TPair<FName, ULiveLinkDeviceCapability::FDeviceTableColumnDesc>& ColumnDesc : CapabilityCDO->GetTableColumns())
	{
		TableColumnIdToCapability.Add(ColumnDesc.Key, InCapability);
	}
}


void ULiveLinkDeviceSubsystem::RegisterDeviceClass(TSubclassOf<ULiveLinkDevice> InDeviceClass)
{
	if (!ensure(InDeviceClass))
	{
		return;
	}

	if (KnownDeviceClasses.Find(InDeviceClass))
	{
		ensureMsgf(false, TEXT("Tried to register duplicate device class"));
		return;
	}

	KnownDeviceClasses.Add(InDeviceClass);

	for (UClass* TempClass = InDeviceClass; TempClass; TempClass = TempClass->GetSuperClass())
	{
		for (TArray<FImplementedInterface>::TConstIterator It(TempClass->Interfaces); It; ++It)
		{
			UClass* InterfaceClass = It->Class;
			if (!InterfaceClass || !InterfaceClass->IsChildOf(ULiveLinkDeviceCapability::StaticClass()))
			{
				continue;
			}

			TSubclassOf<ULiveLinkDeviceCapability> Capability = InterfaceClass;

			// All capabilities should have been enumerated prior to device classes.
			ensure(KnownCapabilities.Find(Capability));

			CapabilitiesByDeviceClass.AddUnique(InDeviceClass, Capability);
			DeviceClassesByCapability.AddUnique(Capability, InDeviceClass);
		}
	}
}
