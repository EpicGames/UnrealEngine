// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkHub/LiveLinkHubSessionExtraData.h"
#include "Templates/SubclassOf.h"
#include "LiveLinkHubSessionExtraData_Device.generated.h"


class ULiveLinkDevice;
class ULiveLinkDeviceSettings;
class ULiveLinkDeviceCapability;

USTRUCT()
struct FLiveLinkDevicePreset
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid DeviceGuid;

	UPROPERTY()
	TSubclassOf<ULiveLinkDevice> DeviceClass;

	UPROPERTY(Instanced)
	TObjectPtr<ULiveLinkDeviceSettings> DeviceSettings;

	UPROPERTY(Instanced)
	TMap<TSubclassOf<ULiveLinkDeviceCapability>, TObjectPtr<UObject>> DeviceCapabilityData;
};


UCLASS()
class ULiveLinkHubSessionExtraData_Device : public ULiveLinkHubSessionExtraData
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<FLiveLinkDevicePreset> Devices;
};
