// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "Engine/Engine.h"
#include "LiveLinkDevice_BasicTest.h"
#include "LiveLinkDeviceSubsystem.h"
#include "Misc/AutomationTest.h"
//#include "Tests/Assertions.h"
#include "UObject/ScriptInterface.h"


BEGIN_DEFINE_SPEC(FLiveLinkDeviceSpec, "LiveLinkHub.Devices",
                  EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter)
	ULiveLinkDevice_BasicTest* TestDevice;
END_DEFINE_SPEC(FLiveLinkDeviceSpec);


void FLiveLinkDeviceSpec::Define()
{
	Describe("ULiveLinkDevice_BasicTest", [this]
	{
		BeforeEach([this]
		{
			ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();
			ULiveLinkDeviceSubsystem::FCreateResult CreateResult =
				Subsystem->CreateDeviceOfClass(ULiveLinkDevice_BasicTest::StaticClass());
			TestDevice = CastChecked<ULiveLinkDevice_BasicTest>(CreateResult.GetValue().Device);
		});

		AfterEach([this]
		{
			ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();
			Subsystem->RemoveDevice(TestDevice);
			TestDevice = nullptr;
		});

		It("can be queried for capabilities via base class pointer", [this]
		{
			ULiveLinkDevice* UnknownDevice = TestDevice;

			// Can still query and retrieve the implemented capabilities.
			TestTrueExpr(UnknownDevice->Implements<ULiveLinkDeviceCapability_BasicTest>());

			TScriptInterface<ILiveLinkDeviceCapability_BasicTest> TestCapability =
				TScriptInterface<ILiveLinkDeviceCapability_BasicTest>(UnknownDevice);

			TestNotNull(
				TEXT("TScriptInterface<ILiveLinkDeviceCapability_BasicTest>::GetInterface()"),
				TestCapability.GetInterface()
			);
		});

		It("can handle capability method invocations via base class pointer", [this]
		{
			ULiveLinkDevice* UnknownDevice = TestDevice;

			const int32 RandomValue = FMath::Rand32();
			ILiveLinkDeviceCapability_BasicTest::Execute_SetValue(UnknownDevice, RandomValue);
			TestEqual(
				TEXT("Randomly generated value set through interface"),
				ILiveLinkDeviceCapability_BasicTest::Execute_GetValue(UnknownDevice),
				RandomValue
			);
		});
	});

	Describe("ULiveLinkDeviceSubsystem", [this]
	{
		It("cannot create a device with a settings template of the wrong type", [this]
		{
			ULiveLinkDeviceSubsystem* Subsystem = GEngine->GetEngineSubsystem<ULiveLinkDeviceSubsystem>();

			// Try to create a device with the wrong settings subclass. This should fail.
			AddExpectedMessage(TEXT("Settings template is of wrong class"), ELogVerbosity::Error);

			ULiveLinkDeviceSettings_Invalid* InvalidSettings = NewObject<ULiveLinkDeviceSettings_Invalid>();

			ULiveLinkDeviceSubsystem::FCreateResult InvalidSettingsCreateResult =
				Subsystem->CreateDeviceOfClass(ULiveLinkDevice_BasicTest::StaticClass(), InvalidSettings);

			TestTrueExpr(InvalidSettingsCreateResult.HasError());
		});
	});
}


#endif // #if WITH_DEV_AUTOMATION_TESTS
