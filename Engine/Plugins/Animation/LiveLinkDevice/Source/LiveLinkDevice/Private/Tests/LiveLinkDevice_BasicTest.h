// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkDevice.h"
#include "LiveLinkDeviceCapability.h"
#include "LiveLinkDevice_BasicTest.generated.h"


UINTERFACE()
class LIVELINKDEVICE_API ULiveLinkDeviceCapability_BasicTest : public ULiveLinkDeviceCapability
{
	GENERATED_BODY()
};


class LIVELINKDEVICE_API ILiveLinkDeviceCapability_BasicTest : public ILiveLinkDeviceCapability
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category="Live Link Device|Testing")
	int32 GetValue() const;
	virtual int32 GetValue_Implementation() const = 0;

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category="Live Link Device|Testing")
	void SetValue(int32 InValue);
	virtual void SetValue_Implementation(int32 InValue) = 0;
};


UCLASS()
class ULiveLinkDeviceSettings_BasicTest : public ULiveLinkDeviceSettings
{
	GENERATED_BODY()

public:

	ULiveLinkDeviceSettings_BasicTest()
	{
		DisplayName = TEXT("Test Device");
	}
};

UCLASS(NotPlaceable)
class ULiveLinkDevice_BasicTest : public ULiveLinkDevice
	, public ILiveLinkDeviceCapability_BasicTest
{
	GENERATED_BODY()

public:
	//~ Begin ULiveLinkDevice interface
	virtual TSubclassOf<ULiveLinkDeviceSettings> GetSettingsClass() const override { return ULiveLinkDeviceSettings_BasicTest::StaticClass(); }
	virtual EDeviceHealth GetDeviceHealth() const override { return EDeviceHealth::Nominal; }
	virtual FText GetHealthText() const override { return FText::FromString("Test Health"); }
	//~ End ULiveLinkDevice interface

	//~ Begin ILiveLinkDeviceCapability_BaseTest interface
	virtual int32 GetValue_Implementation() const override { return TestValue; }
	virtual void SetValue_Implementation(int32 InValue) override { TestValue = InValue; }
	//~ End ILiveLinkDeviceCapability_BaseTest interface

protected:
	int32 TestValue;
};


/** A type that does not correspond to any device, and is used to test expected failures. */
UCLASS()
class ULiveLinkDeviceSettings_Invalid : public ULiveLinkDeviceSettings
{
	GENERATED_BODY()
};
