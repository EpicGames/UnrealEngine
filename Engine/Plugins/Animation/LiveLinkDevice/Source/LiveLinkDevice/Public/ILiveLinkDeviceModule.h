// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"

class ULiveLinkDevice;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnDeviceSelectionChangedDelegate, ULiveLinkDevice*);

DECLARE_LOG_CATEGORY_EXTERN(LogLiveLinkDevice, Log, All);

class FMenuBuilder;

/**
* The public interface to this module
*/
class ILiveLinkDeviceModule : public IModuleInterface
{

public:

	/**
	* Singleton-like access to this module's interface.  This is just for convenience!
	* Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	*
	* @return Returns singleton instance, loading the module on demand if needed
	*/
	static inline ILiveLinkDeviceModule& Get()
	{
		static const FName ModuleName = "LiveLinkDevice";
		return FModuleManager::LoadModuleChecked<ILiveLinkDeviceModule>(ModuleName);
	}

	/**
	* Gets the device selection changed delegate
	* @return Returns the device selection changed delegate
	*/
	virtual FOnDeviceSelectionChangedDelegate& OnSelectionChanged() = 0;

	/** Populate a menu builder with entries for every device registered with the device subsystem. */
	virtual void CreateDeviceMenuEntries(FMenuBuilder& Builder) = 0;
};
