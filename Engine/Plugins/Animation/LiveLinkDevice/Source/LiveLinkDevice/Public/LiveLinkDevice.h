// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "LiveLinkDevice.generated.h"


class FJsonObject;
struct FLiveLinkDeviceWidgetArguments;
class SWidget;
class ULiveLinkDeviceSubsystem;


UENUM(BlueprintType)
enum class EDeviceHealth : uint8
{
	Nominal = 0,
	Good,
	Info,
	Warning,
	Error,
};


UCLASS(Abstract)
class LIVELINKDEVICE_API ULiveLinkDeviceSettings : public UObject
{
	GENERATED_BODY()

public:

	/** Unique display name for this device */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink Device")
	FString DisplayName;
};


/**
 * Abstract base class for all Live Link devices.
 */
UCLASS(Abstract, Blueprintable)
class LIVELINKDEVICE_API ULiveLinkDevice : public UObject
{
	GENERATED_BODY()

public:
	/** Get the device settings class to be displayed in the details view. */
	virtual TSubclassOf<ULiveLinkDeviceSettings> GetSettingsClass() const PURE_VIRTUAL(ULiveLinkDevice::GetSettingsClass, return ULiveLinkDeviceSettings::StaticClass(););

	/** Human-readable display name for this device. */
	UFUNCTION(BlueprintCallable, Category="Live Link Device")
	virtual FText GetDisplayName() const;

	/** At-a-glance health/"severity" of the device. */
	UFUNCTION(BlueprintCallable, Category="Live Link Device")
	virtual EDeviceHealth GetDeviceHealth() const PURE_VIRTUAL(ULiveLinkDevice::GetDeviceHealth, return EDeviceHealth::Error;);

	/** Human-readable explanation for the current device health. */
	UFUNCTION(BlueprintCallable, Category="Live Link Device")
	virtual FText GetHealthText() const PURE_VIRTUAL(ULiveLinkDevice::GetHealthText, return FText(););

	/**
	 * Generate the Slate content, for this device's row, for the specified column.
	 * The default implementation will delegate to the capability CDO where appropriate.
	 */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(
		const FName InColumnId,
		const FLiveLinkDeviceWidgetArguments& InArgs
	);

public:
	/** Called when the device has been registered with the device manager, with either initial or restored settings. */
	virtual void OnDeviceAdded() { }

	/** Called when the device is removed from the device manager. */
	virtual void OnDeviceRemoved() { }

	/** Called when a setting value has changed via the UI. */
	virtual void OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent) { }

	/** Called when the device is being saved to the settings */
	virtual TMap<TSubclassOf<class ULiveLinkDeviceCapability>, TObjectPtr<UObject>> OnSaveDeviceCapabilityData() const { return {}; }

	/** Called when the device is being loaded from the settings */
	virtual void OnLoadDeviceCapabilityData(TMap<TSubclassOf<class ULiveLinkDeviceCapability>, TObjectPtr<UObject>> InCapabilityDataMap) { }

	/** This device's settings UObject. */
	template<typename T = ULiveLinkDeviceSettings>
	const T* GetDeviceSettings() const { return CastChecked<T>(Settings); }

	/** This device's settings UObject (mutable). */
	template<typename T = ULiveLinkDeviceSettings>
	T* GetDeviceSettings() { return CastChecked<T>(Settings); }

	/** The identifier with which this device was registered with `ULiveLinkDeviceSubsystem`. */
	FGuid GetDeviceId() const { return DeviceGuid; }

private:
	friend ULiveLinkDeviceSubsystem;
	virtual void InternalDeviceAdded(const FGuid InDeviceGuid, ULiveLinkDeviceSettings* InSettings);
	/** Provides an image for the status column in the combined list view. */
	const struct FSlateBrush* GetHealthImage() const;

	FGuid DeviceGuid;

	UPROPERTY()
	TObjectPtr<ULiveLinkDeviceSettings> Settings;
};
