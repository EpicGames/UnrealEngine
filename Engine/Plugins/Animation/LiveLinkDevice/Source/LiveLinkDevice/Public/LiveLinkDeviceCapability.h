// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "Widgets/Views/SHeaderRow.h"
#include "LiveLinkDeviceCapability.generated.h"


class SWidget;
class ULiveLinkDevice;
class ULiveLinkDeviceCapability;


/** Helper passed to GenerateWidgetForColumn. */
struct FLiveLinkDeviceWidgetArguments
{
	TDelegate<bool()> IsRowSelected;
};


/**
 * Base class for all device capabilities.
 *
 * Every IInterface has a corresponding UInterface, which exists only as a
 * class default object (CDO), and provides a place to manage centralized
 * state related to the capability.
 */
UINTERFACE(Blueprintable)
class LIVELINKDEVICE_API ULiveLinkDeviceCapability : public UInterface
{
	GENERATED_BODY()

public:
	struct FDeviceTableColumnDesc
	{
		// TODO: Default show/hide, sorting hint, ?

		// This is a global column, no header should be generated for it since it's present by default.
		bool bGlobal = false;
	};

public:
	ULiveLinkDeviceCapability()
	{
		ensure(HasAllFlags(RF_ClassDefaultObject));
	}

	/** Called at completion of Live Link device subsystem initialization. */
	virtual void OnDeviceSubsystemInitialized() {}

	/** Called at the beginning of Live Link device subsystem de-initialization. */
	virtual void OnDeviceSubsystemDeinitializing() {}

	/** Return device table widget columns this capability provides. */
	const TMap<FName, FDeviceTableColumnDesc>& GetTableColumns() const
	{
		return TableColumns;
	}

	/** Configure the header for the specified column. */
	virtual SHeaderRow::FColumn::FArguments& GenerateHeaderForColumn(
		const FName InColumnId,
		SHeaderRow::FColumn::FArguments& InArgs
	)
	{
		// This base class implementation should not get called.
		// If your capability defines table columns, you need to implement this.
		ensure(false);
		return InArgs;
	}

	/** Optional; allows a capability to provide a default widget the device can fall back to. */
	virtual TSharedPtr<SWidget> GenerateWidgetForColumn(
		const FName InColumnId,
		const FLiveLinkDeviceWidgetArguments& InArgs,
		ULiveLinkDevice* InDevice
	)
	{
		return nullptr;
	}

protected:
	/**
	 * Call this from your derived class to define a new device table column.
	 *
	 * @return The full, namespaced column identifier for future reference.
	 */
	FName RegisterTableColumn(const FName InColumnShortName)
	{
		const FName ColumnId = ExpandColumnShortName(InColumnShortName);
		TableColumns.Emplace(ColumnId, {});
		return ColumnId;
	}

	/**
	 * Call this from your derived class to register a handler for one of the existing columns in the combined subjects/sources/devices view.
	 * For example, this can be used by a capability to allow generating a widget for the status or action column.
	 */
	void RegisterGlobalTableColumnHandler(const FName InGlobalColumnName)
	{
		FDeviceTableColumnDesc ColumnDesc;
		ColumnDesc.bGlobal = true;

		TableColumns.Emplace(InGlobalColumnName, ColumnDesc);
	}

	FName ExpandColumnShortName(FName InShortName) const
	{
		FNameBuilder NameBuilder;
		GetClass()->GetFName().AppendString(NameBuilder);
		NameBuilder.AppendChar('.');
		InShortName.AppendString(NameBuilder);
		return FName(NameBuilder.ToView());
	}

private:
	TMap<FName, FDeviceTableColumnDesc> TableColumns;
};


class LIVELINKDEVICE_API ILiveLinkDeviceCapability
{
	GENERATED_BODY()
};
