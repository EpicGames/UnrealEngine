// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkDeviceCapability.h"
#include "LiveLinkDeviceCapability_Connection.generated.h"


/** Device connection states. */
UENUM()
enum class ELiveLinkDeviceConnectionStatus : uint8
{
	Disconnected,
	Connecting,
	Connected,
	Disconnecting,
};


/** \see ILiveLinkDeviceCapability_Connection. */
UINTERFACE()
class LIVELINKDEVICE_API ULiveLinkDeviceCapability_Connection : public ULiveLinkDeviceCapability
{
	GENERATED_BODY()

public:
	const FName Column_HardwareId;

	ULiveLinkDeviceCapability_Connection();

	//~ Begin ULiveLinkDeviceCapability interface
	virtual SHeaderRow::FColumn::FArguments& GenerateHeaderForColumn(
		const FName InColumnId,
		SHeaderRow::FColumn::FArguments& InArgs
	) override;

	virtual TSharedPtr<SWidget> GenerateWidgetForColumn(
		const FName InColumnId,
		const FLiveLinkDeviceWidgetArguments& InArgs,
		ULiveLinkDevice* InDevice
	) override;

	//~ End ULiveLinkDeviceCapability interface

protected:
	/** Builds the menu for the column header. */
	TSharedRef<SWidget> Header_GetMenuContent();

	/** Invokes the `Connect` method on all registered devices implementing this interface. */
	void ConnectAllDevices();

	/** Invokes the `Disconnect` method on all registered devices implementing this interface. */
	void DisconnectAllDevices();
};


DECLARE_TS_MULTICAST_DELEGATE_OneParam(FDeviceConnectionStatusChanged, ELiveLinkDeviceConnectionStatus);
UDELEGATE()
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FDeviceConnectionStatusChangedDynamic, ELiveLinkDeviceConnectionStatus, InNewStatus);


UCLASS()
class LIVELINKDEVICE_API UConnectionDelegate : public UObject
{
	GENERATED_BODY()

public:

	/** Dynamic delegate to be used in Blueprints. Avoid using it outside of the Game Thread. */
	UPROPERTY(EditAnywhere, Category = "Live Link Device|Connection")
	FDeviceConnectionStatusChangedDynamic ConnectionChangedDynamic;

	FDeviceConnectionStatusChanged ConnectionChanged;
};


/**
 * Status and operations relevant to devices with the concept of being (dis)connected.
 * This could be a physical hardware connection, a network connection, or something else.
 */
class LIVELINKDEVICE_API ILiveLinkDeviceCapability_Connection : public ILiveLinkDeviceCapability
{
	GENERATED_BODY()

public:
	ILiveLinkDeviceCapability_Connection();

	/** Get the current connection state. */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category="Live Link Device|Connection")
	ELiveLinkDeviceConnectionStatus GetConnectionStatus() const;
	virtual ELiveLinkDeviceConnectionStatus GetConnectionStatus_Implementation() const = 0;

	/** Retrieve hardware identifier (serial number, network endpoint, etc). */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category="Live Link Device|Connection")
	FString GetHardwareId() const;
	virtual FString GetHardwareId_Implementation() const = 0;

	/** Returns whether it's valid to call `SetHardwareId()` on this device at this time. */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category="Live Link Device|Connection")
	bool CanSetHardwareId();
	virtual bool CanSetHardwareId_Implementation() { return false; }

	/** Set hardware identifier (serial number, network endpoint, etc). */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category="Live Link Device|Connection")
	bool SetHardwareId(const FString& HardwareID);
	virtual bool SetHardwareId_Implementation(const FString& HardwareID) { return false; }

	/** Attempt to establish a connection. */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category="Live Link Device|Connection")
	bool Connect();
	virtual bool Connect_Implementation() = 0;

	/** Attempt to terminate an existing connection. */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category="Live Link Device|Connection")
	bool Disconnect();
	virtual bool Disconnect_Implementation() = 0;

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category="Live Link Device|Connection")
	UConnectionDelegate* GetConnectionDelegate();
	UConnectionDelegate* GetConnectionDelegate_Implementation();

protected:
	virtual void SetConnectionStatus(ELiveLinkDeviceConnectionStatus InStatus);

private:
	TStrongObjectPtr<UConnectionDelegate> ConnectionDelegate;
};
