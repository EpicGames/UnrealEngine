// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkDeviceCapability.h"
#include "LiveLinkDeviceCapability_Recording.generated.h"


UINTERFACE()
class LIVELINKDEVICE_API ULiveLinkDeviceCapability_Recording : public ULiveLinkDeviceCapability
{
	GENERATED_BODY()

public:
	const FName Column_RecordingStatus;

	ULiveLinkDeviceCapability_Recording();

	virtual void OnDeviceSubsystemInitialized() override;
	virtual void OnDeviceSubsystemDeinitializing() override;

	virtual SHeaderRow::FColumn::FArguments& GenerateHeaderForColumn(
		const FName InColumnId,
		SHeaderRow::FColumn::FArguments& InArgs
	) override;
	virtual TSharedPtr<SWidget> GenerateWidgetForColumn(
		const FName InColumnId,
		const FLiveLinkDeviceWidgetArguments& InArgs,
		ULiveLinkDevice* InDevice
	) override;

protected:
	void HandleRecordingStarted();
	void HandleRecordingStopped();
};


class LIVELINKDEVICE_API ILiveLinkDeviceCapability_Recording : public ILiveLinkDeviceCapability
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category="Live Link Device|Recording")
	bool StartRecording();
	virtual bool StartRecording_Implementation() = 0;

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category="Live Link Device|Recording")
	bool StopRecording();
	virtual bool StopRecording_Implementation() = 0;

	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category="Live Link Device|Recording")
	bool IsRecording() const;
	virtual bool IsRecording_Implementation() const = 0;
};
