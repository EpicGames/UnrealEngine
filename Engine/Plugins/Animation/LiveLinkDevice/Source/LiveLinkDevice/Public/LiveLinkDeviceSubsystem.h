// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkHub/LiveLinkHubSessionExtraData.h"

#include "Misc/Optional.h"
#include "Subsystems/EngineSubsystem.h"
#include "Templates/SubclassOf.h"
#include "Templates/ValueOrError.h"
#include "LiveLinkDeviceSubsystem.generated.h"


class ULiveLinkDevice;
class ULiveLinkDeviceSettings;
class ULiveLinkDeviceCapability;

#define UE_API LIVELINKDEVICE_API

USTRUCT(BlueprintType)
struct FLiveLinkDeviceCreateResult
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category="Live Link|Devices")
	FGuid DeviceId;

	UPROPERTY(BlueprintReadWrite, Category="Live Link|Devices")
	TObjectPtr<ULiveLinkDevice> Device;
};


USTRUCT(BlueprintType)
struct FLiveLinkDeviceCreateError
{
	GENERATED_BODY()
};


/**
 * Device repository with lifecycle notifications.
 * Facilitates cached lookups related to device and capability classes.
 */
UCLASS(MinimalAPI, BlueprintType)
class ULiveLinkDeviceSubsystem : public UEngineSubsystem
	, public ILiveLinkHubSessionExtraDataHandler
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLiveLinkDeviceChangedDelegate, FGuid, ULiveLinkDevice*);

	using FCreateResult = TValueOrError<FLiveLinkDeviceCreateResult, FLiveLinkDeviceCreateError>;

	//~ Begin USubsystem interface
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	//~ End USubsystem interface

	//~ Begin ILiveLinkHubSessionExtraDataHandler interface
	UE_API virtual TSubclassOf<ULiveLinkHubSessionExtraData> GetExtraDataClass() const override;
	UE_API virtual void OnExtraDataSessionSaving(ULiveLinkHubSessionExtraData* InExtraData) override;
	UE_API virtual void OnExtraDataSessionLoaded(const ULiveLinkHubSessionExtraData* InExtraData) override;
	//~ End ILiveLinkHubSessionExtraDataHandler interface

	/**
	 * Register a newly created \c ULiveLinkDevice.
	 * @return A new GUID serving as a handle to this device.
	 */
	UE_API FCreateResult CreateDeviceOfClass(
		TSubclassOf<ULiveLinkDevice> InDeviceClass,
		ULiveLinkDeviceSettings* InDeviceSettingsTemplate = nullptr
	);

	/** Remove a previously registered device. */
	UE_API void RemoveDevice(ULiveLinkDevice* InDevice);

	/** Remove a previously registered device by its GUID. */
	UE_API void RemoveDevice(FGuid InDeviceId);

	FOnLiveLinkDeviceChangedDelegate& OnDeviceAdded() { return OnDeviceAddedDelegate; }
	FOnLiveLinkDeviceChangedDelegate& OnDeviceRemoved() { return OnDeviceRemovedDelegate; }

	/** @return All added ULiveLinkDevice instances and their corresponding GUIDs. */
	const TMap<FGuid, TObjectPtr<ULiveLinkDevice>>& GetDeviceMap() const { return Devices; }

	/** @return All registered device capabilities. */
	const TSet<TSubclassOf<ULiveLinkDeviceCapability>>& GetKnownCapabilities() const { return KnownCapabilities; }

	/** @return All registered device classes. */
	const TSet<TSubclassOf<ULiveLinkDevice>>& GetKnownDeviceClasses() const { return KnownDeviceClasses; }

	/** @return A map from device classes to all capabilities implemented by that device class. */
	const TMultiMap<TSubclassOf<ULiveLinkDevice>, TSubclassOf<ULiveLinkDeviceCapability>>& GetCapabilitiesByDeviceClass() { return CapabilitiesByDeviceClass; }

	/** @return A map from capability classes to all device classes implementing that capability. */
	const TMultiMap<TSubclassOf<ULiveLinkDeviceCapability>, TSubclassOf<ULiveLinkDevice>>& GetDeviceClassesByCapability() { return DeviceClassesByCapability; }

	/** @return A map from device table column to capability class. */
	const TMap<FName, TSubclassOf<ULiveLinkDeviceCapability>>& GetTableColumnIdToCapability() { return TableColumnIdToCapability; }

	UFUNCTION(BlueprintPure, Category="Live Link|Devices", meta=(DeterminesOutputType="DeviceClass"))
	UE_API void GetDevicesByClass(TSubclassOf<ULiveLinkDevice> DeviceClass, TArray<ULiveLinkDevice*>& OutDevices) const;

	UFUNCTION(BlueprintPure, Category="Live Link|Devices")
	UE_API void GetDevicesByCapability(TSubclassOf<ULiveLinkDeviceCapability> Capability, TArray<ULiveLinkDevice*>& OutDevices) const;

	/** Check whether a display name is already in use by any device. */
	UE_API bool IsDisplayNameInUse(const FString& InDisplayName) const;
	
	/** Generate a unique display name by appending a numeric suffix if needed. */
	UE_API FString GenerateUniqueDisplayName(const FString& InBaseName) const;

private:
	FCreateResult InternalCreateDeviceOfClass(
		TSubclassOf<ULiveLinkDevice> InDeviceClass,
		ULiveLinkDeviceSettings* InDeviceSettingsTemplate = nullptr,
		TOptional<FGuid> InDeviceGuid = {}
	);

	void InternalAddDevice(FGuid InDeviceId, ULiveLinkDevice* InDevice, ULiveLinkDeviceSettings* InSettings);
	void InternalRemoveDevice(FGuid InDeviceId, ULiveLinkDevice* InDevice);

	void RegisterCapabilityClass(TSubclassOf<ULiveLinkDeviceCapability> InCapability);
	void RegisterDeviceClass(TSubclassOf<ULiveLinkDevice> InDeviceClass);

private:
	/** Devices by ID (strong references). */
	UPROPERTY()
	TMap<FGuid, TObjectPtr<ULiveLinkDevice>> Devices;

	/** Devices by class. */
	TMultiMap<TSubclassOf<ULiveLinkDevice>, ULiveLinkDevice*> DevicesByClass;

	FOnLiveLinkDeviceChangedDelegate OnDeviceAddedDelegate;
	FOnLiveLinkDeviceChangedDelegate OnDeviceRemovedDelegate;

	//////////////////////////////////////////////////////////////////////////
	// UClass/UInterface metadata
	TSet<TSubclassOf<ULiveLinkDeviceCapability>> KnownCapabilities;
	TSet<TSubclassOf<ULiveLinkDevice>> KnownDeviceClasses;
	TMultiMap<TSubclassOf<ULiveLinkDevice>, TSubclassOf<ULiveLinkDeviceCapability>> CapabilitiesByDeviceClass;
	TMultiMap<TSubclassOf<ULiveLinkDeviceCapability>, TSubclassOf<ULiveLinkDevice>> DeviceClassesByCapability;

	TMap<FName, TSubclassOf<ULiveLinkDeviceCapability>> TableColumnIdToCapability;
};

#undef UE_API